// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetTagItem.h"

#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "AssetTagsEditor"

class SAssetTagItemToolTip : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SAssetTagItemToolTip)
	{}

		/** Binding to get the display name of this asset tag item (must be set) */
		SLATE_ATTRIBUTE(FText, DisplayName)

		/** Callback used to build the tooltip info box for this asset tag item */
		SLATE_EVENT(FOnBuildAssetTagItemToolTipInfo, OnBuildToolTipInfo)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs)
	{
		DisplayName = InArgs._DisplayName;
		OnBuildToolTipInfo = InArgs._OnBuildToolTipInfo;

		SToolTip::Construct(
			SToolTip::FArguments()
			.TextMargin(1.0f)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ToolTipBorder"))
			);
	}

	virtual bool IsEmpty() const override
	{
		return false;
	}

	virtual void OnOpening() override
	{
		SetContentWidget(CreateToolTipWidget());
	}

private:
	TSharedRef<SWidget> CreateToolTipWidget()
	{
		TSharedRef<SVerticalBox> OverallTooltipVBox = SNew(SVerticalBox);

		// Create a box to hold every line of info in the body of the tooltip
		TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);
		if (OnBuildToolTipInfo.IsBound())
		{
			OnBuildToolTipInfo.Execute([&InfoBox](const FText& Key, const FText& Value)
			{
				AddToToolTipInfoBox(InfoBox, Key, Value);
			});
		}

		// Top section (asset tag item name)
		OverallTooltipVBox->AddSlot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SBorder)
				.Padding(6)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
				[
					SNew(STextBlock)
					.Text(DisplayName)
					.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
				]
			];

		// Bottom section (asset tag item details, if provided)
		if (InfoBox->NumSlots() > 0)
		{
			OverallTooltipVBox->AddSlot()
				.AutoHeight()
				.Padding(0, 4, 0, 0)
				[
					SNew(SBorder)
					.Padding(6)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						InfoBox
					]
				];
		}

		return SNew(SBorder)
			.Padding(6)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
			[
				OverallTooltipVBox
			];
	}

	static void AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value)
	{
		InfoBox->AddSlot()
		.AutoHeight()
		.Padding(0, 1)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("AssetTagTooltipKeyFormat", "{0}:"), Key))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(Value)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.WrapTextAt(700.0f)
			]
		];
	}

	/** Binding to get the display name of this asset tag item (must be set) */
	TAttribute<FText> DisplayName;

	/** Callback used to build the tooltip info box for this asset tag item */
	FOnBuildAssetTagItemToolTipInfo OnBuildToolTipInfo;
};

void SAssetTagItem::Construct(const FArguments& InArgs)
{
	BaseColor = InArgs._BaseColor;
	WarningText = InArgs._WarningText;
	IsSelectedCallback = InArgs._IsSelected;

	checkf(InArgs._DisplayName.IsSet(), TEXT("SAssetTagItem DisplayName must be set!"));

	TSharedRef<SAssetTagItemToolTip> AssetTagToolTip = SNew(SAssetTagItemToolTip)
		.DisplayName(InArgs._DisplayName)
		.OnBuildToolTipInfo(InArgs._OnBuildToolTipInfo);

	const ANSICHAR* StyleSpecifier = nullptr;
	if (InArgs._ViewMode == EAssetTagItemViewMode::Compact)
	{
		StyleSpecifier = ".Compact";
	}

	TAttribute<bool> IsCheckBoxEnabled = InArgs._IsCheckBoxEnabled;
	if (!InArgs._IsChecked.IsSet() || !InArgs._OnCheckStateChanged.IsBound())
	{
		IsCheckBoxEnabled = false;
	}

	TSharedPtr<SWidget> NameWidget;
	if (InArgs._OnNameCommitted.IsBound())
	{
		NameWidget = SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
			.Text(InArgs._DisplayName)
			.HighlightText(InArgs._HighlightText)
			.OnBeginTextEdit(InArgs._OnBeginNameEdit)
			.OnTextCommitted(InArgs._OnNameCommitted)
			.OnVerifyTextChanged(InArgs._OnVerifyName)
			.IsSelected(IsSelectedCallback)
			.IsReadOnly(InArgs._IsNameReadOnly);
	}
	else
	{
		NameWidget = SNew(STextBlock)
			.Text(InArgs._DisplayName)
			.HighlightText(InArgs._HighlightText);
	}

	TSharedPtr<SHorizontalBox> HBox;

	ChildSlot
	[
		SAssignNew(HBox, SHorizontalBox)
		.ToolTip(AssetTagToolTip)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(CheckBox, SCheckBox)
			.IsEnabled(IsCheckBoxEnabled)
			.IsChecked(InArgs._IsChecked)
			.OnCheckStateChanged(InArgs._OnCheckStateChanged)
			.ToolTipText(this, &SAssetTagItem::GetCheckBoxTooltipText)
		]

		+SHorizontalBox::Slot()
		.Padding(FAppStyle::GetMargin("ContentBrowser.AssetTagNamePadding", StyleSpecifier))
		.VAlign(VAlign_Center)
		[
			NameWidget.ToSharedRef()
		]
	];

	if (WarningText.IsSet())
	{
		HBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Visibility(this, &SAssetTagItem::GetWarningIconVisibility)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentYellow"))
				.ToolTipText(WarningText)
			];
	}

	if (InArgs._CountText.IsSet())
	{
		HBox->AddSlot()
			.AutoWidth()
			.Padding(FAppStyle::GetMargin("ContentBrowser.AssetTagCountPadding", StyleSpecifier))
			[
				SNew(SBorder)
				.Padding(0.0f)
				.VAlign(VAlign_Center)
				.BorderBackgroundColor(this, &SAssetTagItem::GetCountBackgroundColor)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetTagBackground"))
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.MinDesiredWidth(30.0f)
					[
						SNew(STextBlock)
						.Text(InArgs._CountText)
						.Justification(ETextJustify::Center)
					]
				]
			];
	}
}

#undef LOCTEXT_NAMESPACE
