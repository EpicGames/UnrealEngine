// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"

enum class EAssetTagItemViewMode : uint8
{
	Standard,
	Compact,
};

typedef TFunctionRef<void(const FText&, const FText&)> FOnBuildAssetTagItemToolTipInfoEntry;
DECLARE_DELEGATE_OneParam(FOnBuildAssetTagItemToolTipInfo, const FOnBuildAssetTagItemToolTipInfoEntry&);
