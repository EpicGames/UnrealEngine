// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTagItemTypes.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API ASSETTAGSEDITOR_API

class SWidget;
struct FSlateBrush;


template <typename ItemType>
class SAssetTagItemTableRow;

template <typename ItemType>
struct TWidgetTypeTraits<class SAssetTagItemTableRow<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Custom table row used to contain asset tag items within list and tree views */
template <typename ItemType>
class SAssetTagItemTableRow : public STableRow<ItemType>
{
public:
	void Construct(const typename STableRow<ItemType>::FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		this->ConstructInternal(InArgs, InOwnerTableView);

		this->ConstructChildren(
			InOwnerTableView->TableViewMode,
			ApplySelectionBorderOuterPadding(InArgs._Padding),
			WrapContentInSelectionBorder(InArgs._Content.Widget)
			);
	}

	void SetIsDropTarget(const TAttribute<bool>& InIsDropTarget)
	{
		IsDropTarget = InIsDropTarget;
	}

	virtual void SetContent(TSharedRef<SWidget> InContent) override
	{
		STableRow<ItemType>::SetContent(WrapContentInSelectionBorder(InContent));
	}

	virtual const FSlateBrush* GetBorder() const override
	{
		if (IsDropTarget.Get(false))
		{
			return &this->Style->InactiveHoveredBrush;
		}

		return STableRow<ItemType>::GetBorder();
	}

private:
	TSharedRef<SWidget> WrapContentInSelectionBorder(const TSharedRef<SWidget> InContent) const
	{
		return
			SNew(SBorder)
			.Padding(0)
			.BorderImage(this, &SAssetTagItemTableRow::GetBorder)
			[
				InContent
			];
	}

	TAttribute<FMargin> ApplySelectionBorderOuterPadding(const TAttribute<FMargin>& InPadding) const
	{
		const FMargin ExtraPadding = FMargin(0.0f, 0.0f, 0.0f, 1.0f);

		if (InPadding.IsBound())
		{
			return MakeAttributeLambda([InPadding, ExtraPadding]()
			{
				return InPadding.Get() + ExtraPadding;
			});
		}
		else if (InPadding.IsSet())
		{
			return InPadding.Get() + ExtraPadding;
		}

		return ExtraPadding;
	}

	TAttribute<bool> IsDropTarget;
};

template <>
struct TWidgetTypeTraits<class SAssetTagItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A single asset tag item in the tags and collections tree. */
class SAssetTagItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetTagItem)
		: _ViewMode(EAssetTagItemViewMode::Standard)
		, _BaseColor(FLinearColor::White)
	{}

		/** Should this asset tag item use the standard or compact view? */
		SLATE_ARGUMENT(EAssetTagItemViewMode, ViewMode)

		/** Binding to get the base color of this asset tag item */
		SLATE_ATTRIBUTE(FLinearColor, BaseColor)

		/** Binding to get the display name of this asset tag item (must be set) */
		SLATE_ATTRIBUTE(FText, DisplayName)

		/** Binding to get the count text of this asset tag item (unset to omit the count UI) */
		SLATE_ATTRIBUTE(FText, CountText)

		/** Binding to get the warning text of this asset tag item (unset to omit the warning UI) */
		SLATE_ATTRIBUTE(FText, WarningText)

		/** Binding to get the highlight text of the asset tag view */
		SLATE_ATTRIBUTE(FText, HighlightText)

		/** Binding to check whether the asset tag item name is read-only (disables the name edit UI) */
		SLATE_ATTRIBUTE(bool, IsNameReadOnly)

		/** Binding to check whether the check box of the asset tag item is enabled */
		SLATE_ATTRIBUTE(bool, IsCheckBoxEnabled)

		/** Binding to check whether the check box of the asset tag item is currently in a checked state (unset to always disable the check box) */
		SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)

		/** Callback when the checked state of the asset tag item check box is checked or unchecked (unset to always disable the check box) */
		SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)

		/** Callback when the asset tag item name starts to be edited */
		SLATE_EVENT(FOnBeginTextEdit, OnBeginNameEdit)

		/** Callback when the asset tag item name is committed (unset to omit the name edit UI) */
		SLATE_EVENT(FOnTextCommitted, OnNameCommitted)

		/** Called to validate the asset tag item name (called during changes and during commit) */
		SLATE_EVENT(FOnVerifyTextChanged, OnVerifyName)

		/** Callback to check if the asset tag item is selected (should only be hooked up if a parent widget is handling selection or focus) */
		SLATE_EVENT(FIsSelected, IsSelected)

		/** Callback used to build the tooltip info box for this asset tag item */
		SLATE_EVENT(FOnBuildAssetTagItemToolTipInfo, OnBuildToolTipInfo)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	void RequestRename()
	{
		if (InlineRenameWidget)
		{
			InlineRenameWidget->EnterEditingMode();
		}
	}

private:
	FLinearColor GetAssetTagBrightColor() const
	{
		return BaseColor.Get(FLinearColor::White).CopyWithNewOpacity(0.3f);
	}

	FLinearColor GetAssetTagDullColor() const
	{
		return BaseColor.Get(FLinearColor::White).CopyWithNewOpacity(0.1f);
	}

	FLinearColor GetAssetTagDisabledColor() const
	{
		return BaseColor.Get(FLinearColor::White).CopyWithNewOpacity(0.04f);
	}

	FText GetCheckBoxTooltipText() const
	{
		return CheckBox && CheckBox->IsEnabled()
			? CheckBox->IsChecked() 
				? NSLOCTEXT("AssetTagsEditor", "RemoveAssetSelectionFromAssetTagItem", "Untag the current asset selection")
				: NSLOCTEXT("AssetTagsEditor", "AddAssetSelectionToAssetTagItem", "Tag the current asset selection")
			: FText::GetEmpty();
	}

	FSlateColor GetCountBackgroundColor() const
	{
		return GetAssetTagBrightColor();
	}

	EVisibility GetWarningIconVisibility() const
	{
		FText WarningTextValue = WarningText.Get(FText::GetEmpty());
		return WarningTextValue.IsEmpty()
			? EVisibility::Collapsed
			: EVisibility::Visible;
	}

	/** Binding to get the base color of this asset tag item */
	TAttribute<FLinearColor> BaseColor;

	/** Binding to get the warning text of this asset tag item (unset to omit the warning UI) */
	TAttribute<FText> WarningText;

	/** Callback to check if the asset tag item is selected (should only be hooked up if a parent widget is handling selection or focus) */
	FIsSelected IsSelectedCallback;

	/** The check box for enabling/disabling the asset tag item */
	TSharedPtr<SCheckBox> CheckBox;

	/** The widget to display the name of the asset tag item and allows for renaming */
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;
};

#undef UE_API
